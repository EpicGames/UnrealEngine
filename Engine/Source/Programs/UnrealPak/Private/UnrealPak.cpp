// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealPak.h"
#include "RequiredProgramMainCPPInclude.h"
#include "PakFileUtilities.h"
#include "IPlatformFilePak.h"
#include "ProjectUtilities.h"
#include "StudioTelemetry.h"

IMPLEMENT_APPLICATION(UnrealPak, "UnrealPak");

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	// Allows this program to accept a project argument on the commandline and use project-specific config
	UE::ProjectUtilities::ParseProjectDirFromCommandline(ArgC, ArgV);

	// start up the main loop,
	// add -nopak since we never want to pick up and mount any existing pak files from the project directory
	GEngineLoop.PreInit(ArgC, ArgV, TEXT("-nopak"));

	double StartTime = FPlatformTime::Seconds();

	if (FParse::Param(FCommandLine::Get(), TEXT("Telemetry")))
	{
		// Only start a telemetry session if specified
		FStudioTelemetry::Get().StartSession();
	}
	
	int32 Result = ExecuteUnrealPak(FCommandLine::Get())? 0 : 1;

	FStudioTelemetry::Get().EndSession();

	UE_LOGF(LogPakFile, Display, "UnrealPak executed in %f seconds", FPlatformTime::Seconds() - StartTime);

	if (FParse::Param(FCommandLine::Get(), TEXT("fastexit")))
	{
		FPlatformMisc::RequestExitWithStatus(true, Result);
	}

	GLog->Flush();

	RequestEngineExit(TEXT("UnrealPak Exiting"));

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return Result;
}

