// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealPak : ModuleRules
{
	public UnrealPak(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(new string[] {
			"ApplicationCore",
			"AssetRegistry",
			"Core", 
			"CoreUObject",
			"Json",
			"PakFile",
			"PakFileUtilities",
			"Projects", 
			"RSA",
			"StudioTelemetry",
			"TargetPlatform",
		});

		if (Target.bBuildWithEditorOnlyData)
		{
			DynamicallyLoadedModuleNames.AddRange(new string[] {
				"PerforceSourceControl"
			});
		}
	}
}
