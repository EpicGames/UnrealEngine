// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealPakTarget : TargetRules
{
	public UnrealPakTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealPak";

		bUseXGEController					= false;
		bCompileFreeType					= false;
		bLoggingToMemoryEnabled				= false;
		bUseLoggingInShipping				= true;
		bCompileWithAccessibilitySupport	= false;
		bCompileWithPluginSupport			= true;
		bIncludePluginsForTargetPlatforms	= true;
		bWithServerCode						= false;
		bCompileNavmeshClusterLinks			= false;
		bCompileNavmeshSegmentLinks			= false;
		bCompileRecast						= false;
		bCompileICU 						= false;
		bWithLiveCoding						= false;
		bBuildDeveloperTools				= false;
		bBuildWithEditorOnlyData			= true;
		bCompileAgainstEngine				= false;
		bCompileAgainstCoreUObject			= true;
		bCompileAgainstApplicationCore		= true;
		bUsesSlate							= false;
		bIsBuildingConsoleApplication		= true;
		bForceBuildTargetPlatforms			= true;
		bNeedsExtraShaderFormats			= true;
		bAreTargetSDKVersionsRelevantOverride = false;

		WindowsPlatform.bUseBundledDbgHelp = false;
		WindowsPlatform.bPixProfilingEnabled = false;
		WindowsPlatform.bMergeIdenticalCOMDATs = true;
		WindowsPlatform.bStripUnreferencedSymbols = true;

		bEnableTrace = true;
		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
		GlobalDefinitions.Add("ALLOW_OTHER_PLATFORM_CONFIG=1"); 
	}
}
